// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "SymsLibDumpLog.h"
#include "LaunchEngineLoop.h"   // GEngineLoop

#include "Algo/ForEach.h"
#include "Async/MappedFileHandle.h"
#include "Async/ParallelFor.h"
#include "HAL/PlatformFileManager.h"
#include "Hash/CityHash.h"
#include "Serialization/Archive.h"
#include "Templates/UniquePtr.h"

#include "RequiredProgramMainCPPInclude.h" // required for ue programs

#include <atomic>

#include "symslib.h"

// Only HOST platforms will be running this, which Linux/Mac need/support this type of demangling
#define SUPPORTS_CXXABI_DEMANGLE PLATFORM_LINUX || PLATFORM_MAC

#if SUPPORTS_CXXABI_DEMANGLE
#include <cxxabi.h> // for demangling
#endif

DEFINE_LOG_CATEGORY(LogSymsLibDump);

IMPLEMENT_APPLICATION(SymsLibDump, "SymsLibDump");

namespace
{
	// On the rare chance we dont have a proper name parsed from the dwarf debug we will fallback to this
	const ANSICHAR* UnknownName = "<name omitted>";

	struct FAutoMappedFile
	{
		TUniquePtr<IMappedFileHandle> Handle;
		TUniquePtr<IMappedFileRegion> Region;

		bool Load(const TCHAR* FileName)
		{
			FOpenMappedResult Result = FPlatformFileManager::Get().GetPlatformFile().OpenMappedEx(FileName);
			if (Result.HasValue())
			{
				Handle = Result.StealValue();
				Region.Reset(Handle->MapRegion(0, Handle->GetFileSize()));
			}
			else
			{
				Handle.Reset();
				Region.Reset();
			}
			return Region.IsValid();
		}

		SYMS_String8 GetData() const
		{
			if (Region.IsValid())
			{
				return syms_str8((SYMS_U8*)Region->GetMappedPtr(), Region->GetMappedSize());
			}

			return syms_str8(nullptr, 0);
		}
	};

	// cxa_demangle allocates enough room with malloc if it can demangle. If we we must call free on that pointer
	ANSICHAR* Demangle(const ANSICHAR* MangledName)
	{
#if SUPPORTS_CXXABI_DEMANGLE
		return abi::__cxa_demangle(MangledName, nullptr, nullptr, nullptr);
#endif
		return nullptr;
	}

	struct FSymsLine
	{
		uint64 Address;
		uint64 Size;
		uint32 Line;
		uint32 FileId;
	};

	struct FSymsUnit
	{
		SYMS_SpatialMap1D ProcMap;
		SYMS_String8Array FileTable;
		SYMS_LineTable    LineTable;
		SYMS_SpatialMap1D LineMap;
		SYMS_String8      CompileDir;
	};

	struct FSymsSymbol
	{
		const ANSICHAR* Name;
		uint64 Address;
		uint64 Size;

		// Info that will be fill in later to to associate line info with a function
		FSymsUnit* LineUnit;
		uint64 LineSeqIndex;
	};

	struct FSymsInstance
	{
		TArray<SYMS_Arena*>	Arenas;
		TArray<FSymsUnit>	Units;
		SYMS_SpatialMap1D	UnitMap;
		uint64				DefaultBase;
	};

	struct FSymsExported
	{
		FString	Name;
		uint64	Address;
	};

	// need to store our proper FullFile path (with the CompileDir) + the GlobalIndex for this file
	struct FullFileId
	{
		FString FullFilePath;
		uint64 GlobalIndex;
	};

	bool LoadBinary(const TCHAR* BinaryPath, SYMS_Arena* Arena, SYMS_ParseBundle& Bundle, FAutoMappedFile& BinaryFile)
	{
		if (!BinaryFile.Load(BinaryPath))
		{
			UE_LOGF(LogSymsLibDump, Error, "Failed to load '%ls' binary file", BinaryPath);
			return false;
		}

		SYMS_FileAccel* Accel   = syms_file_accel_from_data(Arena, BinaryFile.GetData());
		SYMS_BinAccel* BinAccel = syms_bin_accel_from_file(Arena, BinaryFile.GetData(), Accel);
		if (!syms_accel_is_good(BinAccel))
		{
			UE_LOGF(LogSymsLibDump, Error, "Cannot parse '%ls' binary file", BinaryPath);
			return false;
		}

		Bundle.bin_data = BinaryFile.GetData();
		Bundle.bin      = BinAccel;

		return true;
	}

	bool LoadDebug(const TCHAR* BinaryPath, SYMS_Arena* Arena, SYMS_ParseBundle& Bundle, const FAutoMappedFile& BinaryFile, FAutoMappedFile& DebugFile)
	{
		if (syms_bin_is_dbg(Bundle.bin))
		{
			// binary has debug info built-in (like dwarf file)
			Bundle.dbg = syms_dbg_accel_from_bin(Arena, BinaryFile.GetData(), Bundle.bin);
			Bundle.dbg_data = Bundle.bin_data;

			return true;
		}

		// we're loading extra file (pdb for exe)
		SYMS_ExtFileList List = syms_ext_file_list_from_bin(Arena, BinaryFile.GetData(), Bundle.bin);
		if (!List.first)
		{
			UE_LOGF(LogSymsLibDump, Error, "Binary file '%ls' built without debug info", BinaryPath);
			return false;
		}

		// TODO make a fancy search path setup where we try to look for other place *besides* right next to the binary
		FString DebugFilePath = FPaths::GetPath(FString(BinaryPath)) / FString(UTF8_TO_TCHAR(List.first->ext_file.file_name.str));

		if (!DebugFile.Load(*DebugFilePath))
		{
			UE_LOGF(LogSymsLibDump, Error, "Failed to load debug file '%ls'", *DebugFilePath);
			return false;
		}

		SYMS_FileAccel* Accel   = syms_file_accel_from_data(Arena, DebugFile.GetData());
		SYMS_BinAccel* BinAccel = syms_bin_accel_from_file(Arena, DebugFile.GetData(), Accel);
		SYMS_DbgAccel* DbgAccel = syms_dbg_accel_from_bin(Arena, DebugFile.GetData(), BinAccel);
		if (!syms_accel_is_good(DbgAccel))
		{
			UE_LOGF(LogSymsLibDump, Error, "Cannot parse '%ls' debug file", *DebugFilePath);
			return false;
		}

		Bundle.dbg = DbgAccel;
		Bundle.dbg_data = DebugFile.GetData();

		return true;
	}

	// Most of this was taken from SymslibResolver.cpp
	bool ParseDebugInfo(SYMS_Group* Group, SYMS_ParseBundle& Bundle, FSymsInstance& OutInstance)
	{
		// init group with bundle
		syms_set_lane(0);
		syms_group_init(Group, &Bundle);

		SYMS_U64 UnitCount = syms_group_unit_count(Group);
		OutInstance.Units.SetNum(UnitCount);

		// per-thread arena storage (at least one)
		int32 WorkerThreadCount = FMath::Max(1, FTaskGraphInterface::Get().GetNumWorkerThreads() + 1);
		OutInstance.Arenas.SetNum(WorkerThreadCount);

		// how many symbols are loaded
		std::atomic<uint32> SymbolCount = 0;

		// parse debug info in multiple threads
		{
			uint32 LaneSlot = FPlatformTLS::AllocTlsSlot();

			std::atomic<uint32> LaneCount = 0;
			syms_group_begin_multilane(Group, WorkerThreadCount);
			ParallelFor(UnitCount, [&OutInstance, Group, LaneSlot, &LaneCount, &SymbolCount](uint32 Index)
			{
				SYMS_Arena* Arena = nullptr;
				uint32 LaneValue = uint32(reinterpret_cast<intptr_t>(FPlatformTLS::GetTlsValue(LaneSlot)));
				if (LaneValue == 0)
				{
					// first time we are on this thread
					LaneValue = ++LaneCount;
					FPlatformTLS::SetTlsValue(LaneSlot, reinterpret_cast<void*>(intptr_t(LaneValue)));

					// syms lane index is 0-based
					uint32 LaneIndex = LaneValue - 1;
					syms_set_lane(LaneIndex);
					Arena = OutInstance.Arenas[LaneIndex] = syms_arena_alloc();
				}
				else
				{
					uint32 LaneIndex = LaneValue - 1;
					syms_set_lane(LaneIndex);
					Arena = OutInstance.Arenas[LaneIndex];
				}

				SYMS_ArenaTemp Scratch = syms_get_scratch(0, 0);

				SYMS_UnitID UnitID = static_cast<SYMS_UnitID>(Index) + 1; // syms unit id's are 1-based
				FSymsUnit* Unit = &OutInstance.Units[Index];

				SYMS_SpatialMap1D* ProcSpatialMap = syms_group_proc_map_from_uid(Group, UnitID);
				Unit->ProcMap = syms_spatial_map_1d_copy(Arena, ProcSpatialMap);

				SYMS_String8Array* FileTable = syms_group_file_table_from_uid_with_fallbacks(Group, UnitID);
				Unit->FileTable = syms_string_array_copy(Arena, 0, FileTable);

				SYMS_LineParseOut* LineParse = syms_group_line_parse_from_uid(Group, UnitID);
				Unit->LineTable = syms_line_table_with_indexes_from_parse(Arena, LineParse);

				SYMS_SpatialMap1D* LineSpatialMap = syms_group_line_sequence_map_from_uid(Group, UnitID);
				Unit->LineMap = syms_spatial_map_1d_copy(Arena, LineSpatialMap);

				SYMS_UnitNames UnitNames = syms_group_unit_names_from_uid(Arena, Group, UnitID);
				Unit->CompileDir = UnitNames.compile_dir;

				SYMS_UnitAccel* UnitAccel = syms_group_unit_from_uid(Group, UnitID);

				SYMS_IDMap ProcIdMap = syms_id_map_alloc(Scratch.arena, 4093);

				SYMS_SymbolIDArray* ProcArray = syms_group_proc_sid_array_from_uid(Group, UnitID);
				SYMS_U64 ProcCount = ProcArray->count;

				FSymsSymbol* Symbols = syms_push_array(Arena, FSymsSymbol, ProcCount);
				for (SYMS_U64 ProcIndex = 0; ProcIndex < ProcCount; ProcIndex++)
				{
					SYMS_SymbolID SymbolID = ProcArray->ids[ProcIndex];

					SYMS_U64RangeArray RangeArray = syms_proc_vranges_from_sid(Arena, Group->dbg_data, Group->dbg, UnitAccel, SymbolID);

					if (RangeArray.count > 0)
					{
						Symbols[ProcIndex].Address = RangeArray.ranges[0].min;
						Symbols[ProcIndex].Size    = RangeArray.ranges[0].max - RangeArray.ranges[0].min;
					}

					// TODO need new changes from RAD tools
					SYMS_String8 Name {nullptr, 0}; // = syms_linkage_name_from_sid(Arena, Group->dbg_data, Group->dbg, UnitAccel, SymbolID);

					// If we fail to find a linkage name fallback to name from sid. Some platforms, like Windows wont have a linkage name
					if  (Name.size == 0)
					{
						Name = syms_group_symbol_name_from_sid(Arena, Group, UnitAccel, SymbolID);
					}

					// If we have an empty name for some reason lets give it at least a default
					if (Name.str && *Name.str == '\0')
					{
						Symbols[ProcIndex].Name = UnknownName;
					}
					else
					{
						Symbols[ProcIndex].Name = reinterpret_cast<ANSICHAR*>(Name.str);
					}

					Symbols[ProcIndex].LineUnit     = nullptr;
					Symbols[ProcIndex].LineSeqIndex = ~0;

					syms_id_map_insert(Scratch.arena, &ProcIdMap, SymbolID, &Symbols[ProcIndex]);
				}

				SYMS_SpatialMap1D* ProcMap = &Unit->ProcMap;
				for (SYMS_SpatialMap1DRange* Range = ProcMap->ranges, *EndRange = ProcMap->ranges + ProcMap->count; Range < EndRange; Range++)
				{
					void* SymbolPtr = syms_id_map_ptr_from_u64(&ProcIdMap, Range->val);
					Range->val = SYMS_U64(reinterpret_cast<intptr_t>(SymbolPtr));
				}

				syms_release_scratch(Scratch);

				SymbolCount += ProcCount;
			});
			syms_group_end_multilane(Group);

			FPlatformTLS::FreeTlsSlot(LaneSlot);
		}

		SYMS_Arena* Arena = OutInstance.Arenas[0];

		OutInstance.UnitMap     = syms_spatial_map_1d_copy(Arena, syms_group_unit_map(Group));
		OutInstance.DefaultBase = syms_group_default_vbase(Group);

		syms_group_release(Group);

		return true;
	}

	// Go through and gather all PUBLIC functions which some *may* not be part of the debug file
	// This allows us to gather a few more symbols that wont have an <line> info but still give us some
	// function names. Such as _start
	TArray<FSymsExported> GatherExportedFunctions(const SYMS_ExportArray& ExportArray)
	{
		TArray<FSymsExported> ExportedFunctions;

		for (int ExporteIndex = 0; ExporteIndex < ExportArray.count; ExporteIndex++)
		{
			// Demangle allocates with malloc if it demangles. If we we must call free on that pointer
			ANSICHAR* DemangledName = Demangle(reinterpret_cast<ANSICHAR*>(ExportArray.exports[ExporteIndex].name.str));
			if (DemangledName != nullptr)
			{
				ExportedFunctions.Add(FSymsExported{FString(ANSI_TO_TCHAR(DemangledName)), ExportArray.exports[ExporteIndex].address});

				free(DemangledName);
			}
			else
			{
				ExportedFunctions.Add(FSymsExported{
					FString(ANSI_TO_TCHAR(reinterpret_cast<ANSICHAR*>(ExportArray.exports[ExporteIndex].name.str))), ExportArray.exports[ExporteIndex].address
				});
			}
		}

		ExportedFunctions.Sort([] (const FSymsExported& Left, const FSymsExported& Right) {
				return Left.Address < Right.Address;
		});

		return ExportedFunctions;
	}

	// TODO actually figure out the rest of the info needed for a MODULE record
	void OutputModule(const FSymsInstance& Instance, FArchive* Ar)
	{
		// TODO
		// TUtf8StringBuilder<512> OutputLine;

		FAnsiStringView Module = ANSITEXTVIEW("MODULE\n");
		Ar->Serialize((void*)Module.GetData(), Module.Len());
	}

	// Generate a mapping that is used to go over all the exiting file tables *per* compilation units and create a global
	// mapping, where 1 file, has a single unique global ID. symslib only has them per unit. So this mapping
	// will store files, and then later the value will be setup as the *global index* once all files are added
	TMap<uint64, FullFileId> GenerateStringToIdMap(const FSymsInstance& Instance)
	{
		TMap<uint64, FullFileId> StringToIDMapping;

		for (const FSymsUnit& Unit : Instance.Units)
		{
			for (int FileTableIdx = 0; FileTableIdx < Unit.FileTable.count; FileTableIdx++)
			{
				if (Unit.FileTable.strings)
				{
					FString File(UTF8_TO_TCHAR(Unit.FileTable.strings[FileTableIdx].str));

					uint64 FileHash = CityHash64(reinterpret_cast<ANSICHAR*>(Unit.FileTable.strings[FileTableIdx].str), Unit.FileTable.strings[FileTableIdx].size);

					// hash file (before compiledir), use that as a key, then store a struct with:
					//   - full file path
					//   - global index

					// If we are not an absolute path lets combine with the compile dir for hopefully a full absolute path
					if (FPaths::IsRelative(File))
					{
						File = FString(UTF8_TO_TCHAR(Unit.CompileDir.str)) / File;
					}

					// Need to make sure when looking up later to replace \ with /
					File.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);

					// FindOrAdd to prevent overwriting existing ones, kind of nasty as we have multiple files with different compile dirs...
					// and honestly some are just wrong to use how we are using here. Need to think on this, or possibly just avoid using CompileDir
					StringToIDMapping.FindOrAdd(FileHash, FullFileId{File, 0});
				}
			}
		}

		// Once we have the global string map setup we can then get a list of the keys
		// which will then be used to set the GlobalIndex *per* File
		TArray<uint64> FileList;
		StringToIDMapping.GetKeys(FileList);

		for (uint64 Index = 0; Index < FileList.Num(); Index++)
		{
			StringToIDMapping[FileList[Index]].GlobalIndex = Index;
		}

		return StringToIDMapping;
	}

	// Using the StringToIDMapping, output our new fixed up global file table
	void OutputFileTable(const TMap<uint64, FullFileId>& StringToIDMapping, FArchive* Ar)
	{
		TUtf8StringBuilder<512> OutputLine;

		for (const TPair<uint64, FullFileId>& StringMap : StringToIDMapping)
		{
			OutputLine.Reset();
			OutputLine << "FILE " << StringMap.Value.GlobalIndex << " " << StringMap.Value.FullFilePath << LINE_TERMINATOR_ANSI;

			Ar->Serialize(OutputLine.GetData(), OutputLine.Len());
		}
	}

	// Each FUNC record needs to output its <line> info right after its outputted.
	// So for each <line> record, find the FUNC associated with it such that when we iterate over all the
	// FUNC records we can then output all the <line> info
	void AssociateLineUnitsWithFuncs(FSymsInstance& Instance)
	{
		for (const FSymsUnit& Unit : Instance.Units)
		{
			for (SYMS_U64 SeqIdx = 0; SeqIdx < Unit.LineTable.sequence_count; SeqIdx++)
			{
				SYMS_U64 First = Unit.LineTable.sequence_index_array[SeqIdx];
				SYMS_Line* Line = Unit.LineTable.line_array + First;

				SYMS_UnitID UnitID = syms_spatial_map_1d_value_from_point(&Instance.UnitMap, Line->voff);
				if (UnitID)
				{
					FSymsUnit* FoundUnit = &Instance.Units[UnitID - 1];

					SYMS_U64 Value = syms_spatial_map_1d_value_from_point(&FoundUnit->ProcMap, Line->voff);
					if (Value)
					{
						FSymsSymbol* Syms = reinterpret_cast<FSymsSymbol*>(Value);

						// Will be used later to iterate over all the line info for each FUNC that has this line info setup
						Syms->LineUnit     = FoundUnit;
						Syms->LineSeqIndex = SeqIdx;
					}
				}
			}
		}
	}

	// Attempt to demangle the Syms->Name, if it can be use that name which should be a fully qualified name
	void OutputFunc(const FSymsSymbol* Syms, uint64 BaseOffset, FArchive* Ar)
	{
		TUtf8StringBuilder<512> OutputLine;

		const ANSICHAR* FunctionName = Syms->Name;
		ANSICHAR* DemangledName      = Demangle(Syms->Name);
		bool bDemangled              = false;

		if (DemangledName != nullptr)
		{
			bDemangled   = true;
			FunctionName = DemangledName;
		}

		// FUNC output
		OutputLine.Appendf(UTF8TEXT("FUNC %llx %llx 0 %s" LINE_TERMINATOR_ANSI), Syms->Address - BaseOffset, Syms->Size, FunctionName);

		Ar->Serialize(OutputLine.GetData(), OutputLine.Len());

		// Demangle allocates room with malloc if it demangles. If we we must call free on that pointer
		if (bDemangled)
		{
			free(DemangledName);
		}
	}

	// Output all FUNC records as well as output all <line> records that have been fixed up per FUNC records
	// This gives us a complete FUNC + <line> record output.
	void OutputFuncAndLine(const FSymsInstance& Instance, const TMap<uint64, FullFileId>& StringToIDMapping, FArchive* Ar, TArray<uint64>& OutFuncAddresses)
	{
		TUtf8StringBuilder<512> OutputLine;

		uint64 BaseOffset = Instance.DefaultBase;

		// Output all FUNCs and line info if the FUNC has them
		for (const FSymsUnit& Unit : Instance.Units)
		{
			for (int ProcIdx = 0; ProcIdx < Unit.ProcMap.count; ProcIdx++)
			{
				FSymsSymbol* Syms = reinterpret_cast<FSymsSymbol*>(Unit.ProcMap.ranges[ProcIdx].val);
				if (Syms->Address)
				{
					// Collect these to make sure we only print the required PUBLIC symbols later
					OutFuncAddresses.Add(Syms->Address);

					OutputFunc(Syms, BaseOffset, Ar);

					if (Syms->LineUnit)
					{
						SYMS_U64 First       = Syms->LineUnit->LineTable.sequence_index_array[Syms->LineSeqIndex];
						SYMS_U64 OnePastLast = Syms->LineUnit->LineTable.sequence_index_array[Syms->LineSeqIndex + 1];

						SYMS_Line* Line = Syms->LineUnit->LineTable.line_array + First;

						// we can compress line records if they share the same file id + line number
						uint64 CompressedStartAddress = 0;
						uint64 CompressedTotalSize    = 0;

						for (SYMS_U64 LineIdx = First; LineIdx < OnePastLast - 1; LineIdx++)
						{
							Line                = Syms->LineUnit->LineTable.line_array + LineIdx;
							SYMS_Line* LineNext = Syms->LineUnit->LineTable.line_array + LineIdx + 1;

							uint64 Size    = LineNext->voff - Line->voff;
							uint64 Address = Line->voff - BaseOffset;

							// honestly not sure why the FileId is - 1 vs just being the correct index. Something to check with symslib at some point
							uint64 FileId             = Line->src_coord.file_id - 1;
							uint32 LineNumber         = Line->src_coord.line;
							uint64 UnitFileTableCount = Syms->LineUnit->FileTable.count;

							// we have the same line record, lets just compress these entries, and always add the last one!
							if (LineNext->src_coord.file_id - 1 == FileId && LineNext->src_coord.line == LineNumber && LineIdx + 1 < OnePastLast - 1)
							{
								if (CompressedStartAddress == 0)
								{
									CompressedStartAddress = Address;
									CompressedTotalSize    = Size;
								}
								else
								{
									CompressedTotalSize += Size;
								}

								continue;
							}
							// If the next one is not a match, lets update our total size including this line record
							else
							{
								CompressedTotalSize += Size;
							}

							// If we have been bundling multiple line record together lets use the starting address + size of all
							if (CompressedStartAddress != 0)
							{
								Address = CompressedStartAddress;
								Size    = CompressedTotalSize;
							}

							if (FileId < UnitFileTableCount)
							{
								FString FileToLookFor(ANSI_TO_TCHAR(reinterpret_cast<ANSICHAR*>(Syms->LineUnit->FileTable.strings[FileId].str)));
								uint64 FileHash = CityHash64(reinterpret_cast<ANSICHAR*>(Syms->LineUnit->FileTable.strings[FileId].str), Syms->LineUnit->FileTable.strings[FileId].size);

								FileId = StringToIDMapping[FileHash].GlobalIndex;
							}

							// <line> output
							OutputLine.Reset();
							OutputLine.Appendf(UTF8TEXT("%lx %lx %u %lu" LINE_TERMINATOR_ANSI), Address, Size, LineNumber, FileId);

							Ar->Serialize(OutputLine.GetData(), OutputLine.Len());

							// reset our compressed accumulators after we output always
							CompressedStartAddress = 0;
							CompressedTotalSize    = 0;
						}
					}
				}
			}
		}
	}

	// Output all PUBLIC symbols that can not been previously outputted through the FUNC records
	void OutputPublic(const TArray<FSymsExported>& ExportedFunctions, const TArray<uint64>& FuncAddresses, uint64 BaseOffset, FArchive* Ar)
	{
		TUtf8StringBuilder<512> OutputLine;

		for (const FSymsExported& SymExported : ExportedFunctions)
		{
			// Only add this address *if* its not part of our FuncAddress list using a binary search. So it is assumed FuncAddresses is *sorted*
			if (Algo::BinarySearch(FuncAddresses, SymExported.Address) == INDEX_NONE)
			{
				OutputLine.Reset();
				OutputLine.Appendf(UTF8TEXT("PUBLIC %llx 0 %s" LINE_TERMINATOR_ANSI), SymExported.Address - BaseOffset, TCHAR_TO_UTF8(*SymExported.Name));

				Ar->Serialize(OutputLine.GetData(), OutputLine.Len());
			}
		}
	}

	// The goal of this is to *produce* an exact same output file as dump_syms was doing. This is the format:
	// https://chromium.googlesource.com/breakpad/breakpad/+/master/docs/symbol_files.md
	bool ProcessAndProducePortableSymbolFile(const FString& BinaryPath, const FString& OutputFile)
	{
		// temp memory used for loading
		SYMS_Group* Group = syms_group_alloc();

		// contents of the binary & debug file
		SYMS_ParseBundle Bundle;

		// memory-mapped binary & debug file
		FAutoMappedFile BinaryFile;
		FAutoMappedFile DebugFile;

		if (!LoadBinary(*BinaryPath, Group->arena, Bundle, BinaryFile))
		{
			UE_LOGF(LogSymsLibDump, Warning, "Failed to load binary file '%ls'", *BinaryPath);
			return false;
		}

		if (!LoadDebug(*BinaryPath, Group->arena, Bundle, BinaryFile, DebugFile))
		{
			UE_LOGF(LogSymsLibDump, Warning, "Failed to load debug file from binary path '%ls'", *BinaryPath);
			return false;
		}

		// This array will be used for PUBLIC symbols which may not have be in the debug file, such as the _start symbol
		SYMS_ExportArray ExportArray = syms_exports_from_bin(Group->arena, BinaryFile.GetData(), Bundle.bin);
		TArray<FSymsExported> ExportedFunctions = GatherExportedFunctions(ExportArray);

		FSymsInstance Instance;

		if (!ParseDebugInfo(Group, Bundle, Instance))
		{
			return false;
		}

		if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileWriter(*OutputFile)})
		{
			// MODULE
			OutputModule(Instance, Ar.Get());

			// FILE TABLE
			TMap<uint64, FullFileId> StringToIDMapping = GenerateStringToIdMap(Instance);
			OutputFileTable(StringToIDMapping, Ar.Get());

			// FUNC + (line)

			// <line> need to be matched up with a FUNC so its much easier to output in one go
			AssociateLineUnitsWithFuncs(Instance);

			// Gather a list of our Function Address. This is assumed to be sorted as we binary search later on this
			TArray<uint64> FuncAddresses;
			OutputFuncAndLine(Instance, StringToIDMapping, Ar.Get(), FuncAddresses);

			// PUBLIC
			OutputPublic(ExportedFunctions, FuncAddresses, Instance.DefaultBase, Ar.Get());
		}
		else
		{
			UE_LOGF(LogSymsLibDump, Error, "Failed to open '%ls' for writing", *OutputFile);
		}

		// Clean up Arenas, only do once done with the memory
		Algo::ForEach(Instance.Arenas, [] (SYMS_DefArena* Arena) {
			if (Arena != nullptr)
			{
				syms_arena_release(Arena);
			}
		});

		Instance.Arenas.Empty();

		return true;
	}

	int RealMain(const FString& CommandLine)
	{
		FPlatformMisc::SetCrashHandler(nullptr);
		FPlatformMisc::SetGracefulTerminationHandler();

		GEngineLoop.PreInit(*CommandLine);

		ON_SCOPE_EXIT
		{
			FEngineLoop::AppPreExit();
			FEngineLoop::AppExit();
		};

		FString BinaryFile;
		FString OutputFile;

		if (!FParse::Value(*CommandLine, TEXT("input "), BinaryFile))
		{
			UE_LOGF(LogSymsLibDump, Error, "Missing binary file, pass in `-input <path/to/binary>`");
			return -1;
		}

		if (!FParse::Value(*CommandLine, TEXT("output "), OutputFile))
		{
			UE_LOGF(LogSymsLibDump, Error, "Missing output path, pass in `-output <path>`");
			return -1;
		}

		UE_LOGF(LogSymsLibDump, Log, "Binary: '%ls' Output: '%ls'", *BinaryFile, *OutputFile);

		if (!ProcessAndProducePortableSymbolFile(BinaryFile, OutputFile))
		{
			return -1;
		}

		return 0;
	}
} // anonymous namespace

int main(int ArgC, char* ArgV[])
{
	FString CommandLine;

	for (int32 Option = 1; Option < ArgC; Option++)
	{
		CommandLine += TEXT(" ");
		FString Argument(ANSI_TO_TCHAR(ArgV[Option]));
		if (Argument.Contains(TEXT(" ")))
		{
			if (Argument.Contains(TEXT("=")))
			{
				FString ArgName;
				FString ArgValue;
				Argument.Split( TEXT("="), &ArgName, &ArgValue );
				Argument = FString::Printf( TEXT("%s=\"%s\""), *ArgName, *ArgValue );
			}
			else
			{
				Argument = FString::Printf(TEXT("\"%s\""), *Argument);
			}
		}
		CommandLine += Argument;
	}

	return RealMain(CommandLine);
}
