// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms("Win64", "Linux")]
public class SymsLibDumpTarget : TargetRules
{
	public SymsLibDumpTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "SymsLibDump";

		// Make SymsLibDump the shipping version
		UndecoratedConfiguration = UnrealTargetConfiguration.Shipping;

		// Lean and mean
		bBuildDeveloperTools = false;

		// Compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;

		// Logs are still useful to print the results
		bUseLoggingInShipping = true;

		// Make a console application under Windows, so entry point is main() everywhere
		bIsBuildingConsoleApplication = true;

		// Only output on Error or higher logs in shipping builds
		if (Target.Configuration == UnrealTargetConfiguration.Shipping)
		{
			GlobalDefinitions.Add("COMPILED_IN_MINIMUM_VERBOSITY=Error");
		}
	}
}
